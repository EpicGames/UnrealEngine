// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ConsoleSettings.h"
#include "GameMapsSettings.h"
#include "GeneralProjectSettings.h"
#include "EngineGlobals.h"
#include "Widgets/SWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "AI/NavigationSystemBase.h"
#include "NavigationSystem.h"
#include "Engine/Engine.h"
#include "GameFramework/InputSettings.h"
#include "Textures/SlateIcon.h"
#include "Framework/Docking/TabManager.h"
#include "MoviePlayerSettings.h"
#include "Styling/AppStyle.h"
#include "Settings/ProjectPackagingSettings.h"
#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
#include "Interfaces/IProjectTargetPlatformEditorModule.h"
#endif
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsEditorModel.h"
#include "ISettingsEditorModule.h"
#include "ISettingsModule.h"
#include "ISettingsViewer.h"
#include "Widgets/Docking/SDockTab.h"

#include "NavMesh/RecastNavMesh.h"
#include "Navigation/CrowdManager.h"


#include "AISystem.h"
#include "SlateSettings.h"

#define LOCTEXT_NAMESPACE "FProjectSettingsViewerModule"

static const FName ProjectSettingsTabName("ProjectSettings");

/** Holds auto discovered settings information so that they can be unloaded automatically when refreshing. */
struct FRegisteredSettings
{
	FName ContainerName;
	FName CategoryName;
	FName SectionName;
};


/**
 * Implements the ProjectSettingsViewer module.
 */
class FProjectSettingsViewerModule
	: public IModuleInterface
	, public ISettingsViewer
{
public:

	// ISettingsViewer interface

	virtual void ShowSettings( const FName& CategoryName, const FName& SectionName ) override
	{
		FGlobalTabmanager::Get()->TryInvokeTab(ProjectSettingsTabName);
		ISettingsEditorModelPtr SettingsEditorModel = SettingsEditorModelPtr.Pin();

		if (SettingsEditorModel.IsValid())
		{
			ISettingsCategoryPtr Category = SettingsEditorModel->GetSettingsContainer()->GetCategory(CategoryName);

			if (Category.IsValid())
			{
				SettingsEditorModel->SelectSection(Category->GetSection(SectionName));
			}
		}
	}

public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			RegisterEngineSettings(*SettingsModule);
			RegisterProjectSettings(*SettingsModule);

			SettingsModule->RegisterViewer("Project", *this);
		}

		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ProjectSettingsTabName, FOnSpawnTab::CreateRaw(this, &FProjectSettingsViewerModule::HandleSpawnSettingsTab))
			.SetDisplayName(LOCTEXT("ProjectSettingsTabTitle", "Project Settings"))
			.SetMenuType(ETabSpawnerMenuType::Hidden)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon"));
	}

	virtual void ShutdownModule() override
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ProjectSettingsTabName);
		UnregisterSettings();
	}
	
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

protected:

	/**
	 * Registers Engine settings.
	 *
	 * @param SettingsModule A reference to the settings module.
	 */
	void RegisterEngineSettings( ISettingsModule& SettingsModule )
	{
		// startup settings
		SettingsModule.RegisterSettings("Project", "Engine", "General",
			LOCTEXT("GeneralEngineSettingsName", "General Settings"),
			LOCTEXT("ProjectGeneralSettingsDescription", "General options and defaults for the game engine."),
			GetMutableDefault<UEngine>()
		);

		// command console settings
		SettingsModule.RegisterSettings("Project", "Engine", "Console",
			LOCTEXT("ProjectConsoleSettingsName", "Console"),
			LOCTEXT("ProjectConsoleSettingsDescription", "Configure the in-game input console."),
			GetMutableDefault<UConsoleSettings>()
		);

		// input settings
		SettingsModule.RegisterSettings("Project", "Engine", "Input",
			LOCTEXT("EngineInputSettingsName", "Input"),
			LOCTEXT("ProjectInputSettingsDescription", "Input settings, including default input action and axis bindings."),
			GetMutableDefault<UInputSettings>()
		);

		// navigation system's class can be game specific so we need to find appropriate CDO
		UNavigationSystemBase* NavSysCDO = (*GEngine->NavigationSystemClass != nullptr)
			? GetMutableDefault<UNavigationSystemBase>(GEngine->NavigationSystemClass)
			: GetMutableDefault<UNavigationSystemBase>();
		SettingsModule.RegisterSettings("Project", "Engine", "NavigationSystem",
			LOCTEXT("NavigationSystemSettingsName", "Navigation System"),
			LOCTEXT("NavigationSystemSettingsDescription", "Settings for the navigation system."),
			NavSysCDO
		);

		// navigation mesh
		SettingsModule.RegisterSettings("Project", "Engine", "NavigationMesh",
			LOCTEXT("NavigationMeshSettingsName", "Navigation Mesh"),
			LOCTEXT("NavigationMeshSettingsDescription", "Settings for the navigation mesh."),
			GetMutableDefault<ARecastNavMesh>()
		);

		// AI system
		SettingsModule.RegisterSettings("Project", "Engine", "AISystem",
			LOCTEXT("AISystemSettingsName", "AI System"),
			LOCTEXT("AISystemSettingsDescription", "Settings for the AI System."),
			GetMutableDefault<UAISystem>()
			);

		// Crowd manager
		SettingsModule.RegisterSettings("Project", "Engine", "CrowdManager",
			LOCTEXT("CrowdManagerSettingsName", "Crowd Manager"),
			LOCTEXT("CrowdManagerSettingsDescription", "Settings for the AI Crowd Manager."),
			GetMutableDefault<UCrowdManager>()
			);

		SettingsModule.RegisterSettings("Project", "Engine", "Slate",
			LOCTEXT("SlateSettingsName", "Slate Settings"),
			LOCTEXT("SlateSettingsDescription", "Settings for Slate."),
			GetMutableDefault<USlateSettings>()
		);
	}

	/**
	 * Registers Project settings.
	 *
	 * @param SettingsModule A reference to the settings module.
	 */
	void RegisterProjectSettings( ISettingsModule& SettingsModule )
	{
		// general project settings
		SettingsModule.RegisterSettings("Project", "Project", "General",
			LOCTEXT("GeneralGameSettingsName", "Description"),
			LOCTEXT("GeneralGameSettingsDescription", "Descriptions and other information about your project."),
			GetMutableDefault<UGeneralProjectSettings>()
		);

		// map related settings
		SettingsModule.RegisterSettings("Project", "Project", "Maps",
			LOCTEXT("GameMapsSettingsName", "Maps & Modes"),
			LOCTEXT("GameMapsSettingsDescription", "Default maps, game modes and other map related settings."),
			GetMutableDefault<UGameMapsSettings>()
		);

		// packaging settings
		SettingsModule.RegisterSettings("Project", "Project", "Packaging",
			LOCTEXT("ProjectPackagingSettingsName", "Packaging"),
			LOCTEXT("ProjectPackagingSettingsDescription", "Fine tune how your project is packaged for release."),
			GetMutableDefault<UProjectPackagingSettings>()
		);

#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
		// platforms settings
		TWeakPtr<SWidget> ProjectTargetPlatformEditorPanel = FModuleManager::LoadModuleChecked<IProjectTargetPlatformEditorModule>("ProjectTargetPlatformEditor").CreateProjectTargetPlatformEditorPanel();
		SettingsModule.RegisterSettings("Project", "Project", "SupportedPlatforms",
			LOCTEXT("ProjectSupportedPlatformsSettingsName", "Supported Platforms"),
			LOCTEXT("ProjectSupportedPlatformsSettingsDescription", "Specify which platforms your project supports."),
			ProjectTargetPlatformEditorPanel.Pin().ToSharedRef()
		);
#endif

		// movie settings
		SettingsModule.RegisterSettings("Project", "Project", "Movies",
			LOCTEXT("MovieSettingsName", "Movies"),
			LOCTEXT("MovieSettingsDescription", "Movie player settings"),
			GetMutableDefault<UMoviePlayerSettings>()
		);
	}

	/** Unregisters all settings. */
	void UnregisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterViewer("Project");

			// engine settings
			SettingsModule->UnregisterSettings("Project", "Engine", "General");
			SettingsModule->UnregisterSettings("Project", "Engine", "CrowdManager");
			SettingsModule->UnregisterSettings("Project", "Engine", "NavigationSystem");
			SettingsModule->UnregisterSettings("Project", "Engine", "NavigationMesh");
			SettingsModule->UnregisterSettings("Project", "Engine", "Input");
			SettingsModule->UnregisterSettings("Project", "Engine", "Collision");
			SettingsModule->UnregisterSettings("Project", "Engine", "Physics");
			SettingsModule->UnregisterSettings("Project", "Engine", "Rendering");

			// project settings
			SettingsModule->UnregisterSettings("Project", "Project", "General");
			SettingsModule->UnregisterSettings("Project", "Project", "Maps");
			SettingsModule->UnregisterSettings("Project", "Project", "Packaging");
			SettingsModule->UnregisterSettings("Project", "Project", "SupportedPlatforms");
			SettingsModule->UnregisterSettings("Project", "Project", "Movies");

			// Editor settings
			SettingsModule->UnregisterSettings("Editor", "Editor", "Appearance");
		}
	}

private:

	/** Handles creating the project settings tab. */
	TSharedRef<SDockTab> HandleSpawnSettingsTab( const FSpawnTabArgs& SpawnTabArgs )
	{
		TSharedRef<SWidget> SettingsEditor = SNullWidget::NullWidget;
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			ISettingsContainerPtr SettingsContainer = SettingsModule->GetContainer("Project");

			if (SettingsContainer.IsValid())
			{
				ISettingsEditorModule& SettingsEditorModule = FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor");
				ISettingsEditorModelRef SettingsEditorModel = SettingsEditorModule.CreateModel(SettingsContainer.ToSharedRef());

				SettingsEditor = SettingsEditorModule.CreateEditor(SettingsEditorModel);
				SettingsEditorModelPtr = SettingsEditorModel;
			}
		}

		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SettingsEditor
			];
	}

private:

	/** Holds a pointer to the settings editor's view model. */
	TWeakPtr<ISettingsEditorModel> SettingsEditorModelPtr;
};


IMPLEMENT_MODULE(FProjectSettingsViewerModule, ProjectSettingsViewer);


#undef LOCTEXT_NAMESPACE
