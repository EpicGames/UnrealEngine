// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class ProjectSettingsViewer : ModuleRules
	{
		public ProjectSettingsViewer(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"EngineSettings",
					"SettingsEditor",
					"Slate",
					"SlateCore",
					"EditorFramework",
					"UnrealEd",
					"MoviePlayer",
					"NavigationSystem",
					"AIModule",
					"DeveloperToolSettings",
				}
			);

			if (Target.bBuildTargetDeveloperTools)
			{
				PrivateDependencyModuleNames.Add("ProjectTargetPlatformEditor");
			}
		}
	}
}
